// Copyright Epic Games, Inc. All Rights Reserved.

#include "Proxies/TargetDeviceProxy.h"

#include "FileMessageAttachment.h"
#include "HAL/PlatformProcess.h"
#include "MessageEndpointBuilder.h"

#include "TargetDeviceServiceMessages.h"


/* FTargetDeviceProxy structors
 *****************************************************************************/

FTargetDeviceProxy::FTargetDeviceProxy(const FString& InName)
	: Connected(false)
	, Authorized(false)
	, Name(InName)
	, SupportsMultiLaunch(false)
	, SupportsPowerOff(false)
	, SupportsPowerOn(false)
	, SupportsReboot(false)
	, Aggregated(false)
{

	InitializeMessaging();
}


FTargetDeviceProxy::FTargetDeviceProxy(const FString& InName, const FTargetDeviceServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, bool InIsAggregated)
	: Connected(false)
	, Authorized(false)
	, Name(InName)
	, SupportsMultiLaunch(false)
	, SupportsPowerOff(false)
	, SupportsPowerOn(false)
	, SupportsReboot(false)
	, Aggregated(InIsAggregated)
{
	UpdateFromMessage(Message, Context);
	InitializeMessaging();
}


/* FTargetDeviceProxy interface
*****************************************************************************/

void FTargetDeviceProxy::UpdateFromMessage( const FTargetDeviceServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if (Name != (!Aggregated? Message.Name: Message.AllDevicesName))
	{
		return;
	}

	MessageAddress = Context->GetSender();

	Connected = Message.Connected;
	Authorized = Message.Authorized;
	HostName = Message.HostName;
	HostUser = Message.HostUser;
	Make = Message.Make;
	Model = Message.Model;
	Type = Message.Type;
	Name = Aggregated? Message.AllDevicesName: Message.Name;
	DeviceUser = Message.DeviceUser;
	DeviceUserPassword = Message.DeviceUserPassword;
	OSVersion = Message.OSVersion;
	Architecture = Message.Architecture;
	ConnectionType = Message.ConnectionType;
	Shared = Message.Shared;
	SupportsMultiLaunch = Message.SupportsMultiLaunch;
	SupportsPowerOff = Message.SupportsPowerOff;
	SupportsPowerOn = Message.SupportsPowerOn;
	SupportsReboot = Message.SupportsReboot;
	SupportsVariants = Message.SupportsVariants;
	DefaultVariant = Message.DefaultVariant;

	// Update the map of flavors.
	for (int Index = 0; Index < Message.Variants.Num(); Index++)
	{
		const FTargetDeviceVariant& MsgVariant = Message.Variants[Index];

		// create an new entry or add to the existing (an aggregate (All_<platform>_devices_on_<host>) proxy)
		FTargetDeviceProxyVariant & Variant = TargetDeviceVariants.FindOrAdd(MsgVariant.VariantName);
		Variant.DeviceIDs.Add(MsgVariant.DeviceID);
		Variant.VariantName = MsgVariant.VariantName;
		Variant.TargetPlatformName = MsgVariant.TargetPlatformName;
		Variant.TargetPlatformId = MsgVariant.TargetPlatformId;
		Variant.VanillaPlatformId = MsgVariant.VanillaPlatformId;
		Variant.PlatformDisplayName = FText::FromString(MsgVariant.PlatformDisplayName);

		if (Aggregated && MsgVariant.TargetPlatformId.IsEqual(Message.AllDevicesDefaultVariant))
		{
			// for aggregated platforms,check if the declared AllDevicesDefaultVariant is supported by at least one device
			DefaultVariant = Message.AllDevicesDefaultVariant;
		}
	}

	PingTimeout.Reset();
}


/* ITargetDeviceProxy interface
 *****************************************************************************/

int32 FTargetDeviceProxy::GetNumVariants() const
{
	return TargetDeviceVariants.Num();
}


int32 FTargetDeviceProxy::GetVariants(TArray<FName>& OutVariants) const
{
	return TargetDeviceVariants.GetKeys(OutVariants);
}


FName FTargetDeviceProxy::GetTargetDeviceVariant(const FString& InDeviceId) const
{
	for (TMap<FName, FTargetDeviceProxyVariant>::TConstIterator ItVariant(TargetDeviceVariants); ItVariant; ++ItVariant)
	{
		const FTargetDeviceProxyVariant& Variant = ItVariant.Value();

		// for an aggregate (All_<platform>_devices_on_<host>) proxy we have a list of associated devices
		if (TSet<FString>::TConstIterator ItDeviceID(Variant.DeviceIDs); ItDeviceID)
		{
			// should return the first device
			// this method is designed for physical devices, not aggregate proxies
			if (*ItDeviceID == InDeviceId)
			{
				return ItVariant.Key();
			}
		}
	}

	return NAME_None;
}


// for an aggregate (All_<platform>_devices_on_<host>) proxy we have a list of associated devices
const TSet<FString>& FTargetDeviceProxy::GetTargetDeviceIds(FName InVariant) const
{
	if (InVariant == NAME_None)
	{
		return TargetDeviceVariants[DefaultVariant].DeviceIDs;
	}

	return TargetDeviceVariants[InVariant].DeviceIDs;
}

// get the device id for the variant
const FString FTargetDeviceProxy::GetTargetDeviceId(FName InVariant) const
{
	FString Variant;
	// for an aggregate (All_<platform>_devices_on_<host>) proxy we have a list of associated devices
	if (TSet<FString>::TConstIterator ItVariant(TargetDeviceVariants[(InVariant == NAME_None) ? DefaultVariant : InVariant].DeviceIDs); ItVariant)
	{
		// should return the first device
		// this method is designed for physical devices, not aggregate proxies
		Variant = *ItVariant;
	}
	return Variant;
}

FString FTargetDeviceProxy::GetTargetPlatformName(FName InVariant) const
{
	if (InVariant == NAME_None)
	{
		return TargetDeviceVariants[DefaultVariant].TargetPlatformName;
	}

	return TargetDeviceVariants[InVariant].TargetPlatformName;
}


FName FTargetDeviceProxy::GetTargetPlatformId(FName InVariant) const
{
	if (InVariant == NAME_None)
	{
		return TargetDeviceVariants[DefaultVariant].TargetPlatformId;
	}

	return TargetDeviceVariants[InVariant].TargetPlatformId;
}


FName FTargetDeviceProxy::GetVanillaPlatformId(FName InVariant) const
{
	if (InVariant == NAME_None)
	{
		return TargetDeviceVariants[DefaultVariant].VanillaPlatformId;
	}

	return TargetDeviceVariants[InVariant].VanillaPlatformId;
}


FText FTargetDeviceProxy::GetPlatformDisplayName(FName InVariant) const
{
	if (InVariant == NAME_None)
	{
		return TargetDeviceVariants[DefaultVariant].PlatformDisplayName;
	}

	return TargetDeviceVariants[InVariant].PlatformDisplayName;
}


bool FTargetDeviceProxy::HasDeviceId(const FString& InDeviceId) const
{
	// this method is designed for physical devices, not aggregate proxies
	if (Aggregated)
	{
		return false;
	}
	
	for (TMap<FName, FTargetDeviceProxyVariant>::TConstIterator ItVariant(TargetDeviceVariants); ItVariant; ++ItVariant)
	{
		const FTargetDeviceProxyVariant& Variant = ItVariant.Value();

		if (TSet<FString>::TConstIterator ItDeviceID(Variant.DeviceIDs); ItDeviceID)
		{
			// should return the first device
			// this method is designed for physical devices, not aggregate proxies
			if (*ItDeviceID == InDeviceId)
			{
				return true;
			}
		}
	}
	return false;
}


bool FTargetDeviceProxy::HasTargetPlatform(FName InTargetPlatformId) const
{
	for (TMap<FName, FTargetDeviceProxyVariant>::TConstIterator ItVariant(TargetDeviceVariants); ItVariant; ++ItVariant)
	{
		const FTargetDeviceProxyVariant& Variant = ItVariant.Value();

		if (Variant.TargetPlatformId == InTargetPlatformId)
		{
			return true;
		}
	}

	return false;
}


bool FTargetDeviceProxy::HasVariant(FName InVariant) const
{
	if (InVariant == NAME_None)
	{
		return TargetDeviceVariants.Contains(DefaultVariant);
	}

	return TargetDeviceVariants.Contains(InVariant);
}



bool FTargetDeviceProxy::TerminateLaunchedProcess(FName InVariant, const FString& ProcessIdentifier)
{
	MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FTargetDeviceServiceTerminateLaunchedProcess>(InVariant, ProcessIdentifier), MessageAddress);

	return true;
}


void FTargetDeviceProxy::PowerOff(bool Force)
{
	MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FTargetDeviceServicePowerOff>(FPlatformProcess::UserName(false), Force), MessageAddress);
}


void FTargetDeviceProxy::PowerOn()
{
	MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FTargetDeviceServicePowerOn>(FPlatformProcess::UserName(false)), MessageAddress);
}


void FTargetDeviceProxy::Reboot()
{
	MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FTargetDeviceServiceReboot>(FPlatformProcess::UserName(false)), MessageAddress);
}



/* FTargetDeviceProxy implementation
 *****************************************************************************/

void FTargetDeviceProxy::InitializeMessaging()
{
	MessageEndpoint = FMessageEndpoint::Builder(FName(*FString::Printf(TEXT("FTargetDeviceProxy (%s)"), *Name))).Build();
}
