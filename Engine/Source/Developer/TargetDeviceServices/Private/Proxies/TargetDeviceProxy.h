// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IMessageContext.h"
#include "MessageEndpoint.h"
#include "ITargetDeviceProxy.h"
#include "Misc/Optional.h"

class FTargetDeviceProxy;
class IMessageContext;

struct FTargetDeviceServiceDeployFinished;
struct FTargetDeviceServiceLaunchFinished;
struct FTargetDeviceServicePong;


/** Type definition for shared references to instances of FTargetDeviceProxy. */
typedef TSharedRef<class FTargetDeviceProxy> FTargetDeviceProxyRef;


/**
 * Implementation of the device proxy.
 */
class FTargetDeviceProxy
	: public ITargetDeviceProxy
{
public:

	/** Default constructor. */
	FTargetDeviceProxy() { }

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InId The identifier of the target device to create this proxy for.
	 */
	FTargetDeviceProxy(const FString& InId);

	/**
	* Creates and initializes a new instance.
	*
	* @param InId The identifier of the target device to create this proxy for.
	* @param Message The message to initialize from.
	* @param Context The message context.
	* @param InIsAggregated Indicates if the proxy is of type "All Devices".
	*/
	FTargetDeviceProxy(const FString& InName, const FTargetDeviceServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, bool InIsAggregated);

public:

	/**
	 * Add a time by which a ping response should arrive
	 */
	void AddTimeout(const FDateTime& NewTimeout)
	{
		if (!PingTimeout || NewTimeout < PingTimeout.GetValue())
		{
			PingTimeout = NewTimeout;
		}
	}

	/**
	 * Check if ping response arrived on time
	 */
	bool HasTimedOut(const FDateTime& Now) const
	{
		return PingTimeout && Now > PingTimeout.GetValue();
	}

	/**
	 * Updates the proxy's information from the given device service response and clear timeout expectations
	 *
	 * @param Message The message containing the response.
	 * @param Context The message context.
	 */
	void UpdateFromMessage(const FTargetDeviceServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

public:

	// ITargetDeviceProxy interface

	virtual const bool CanMultiLaunch() const override
	{
		return SupportsMultiLaunch;
	}

	virtual bool CanPowerOff() const override
	{
		return SupportsPowerOff;
	}

	virtual bool CanPowerOn() const override
	{
		return SupportsPowerOn;
	}

	virtual bool CanReboot() const override
	{
		return SupportsReboot;
	}

	virtual bool CanSupportVariants() const override
	{
		return SupportsVariants;
	}

	virtual int32 GetNumVariants() const override;
	virtual int32 GetVariants(TArray<FName>& OutVariants) const override;
	virtual FName GetTargetDeviceVariant(const FString& InDeviceId) const override;
	const FString GetTargetDeviceId(FName InVariant) const override;
	virtual const TSet<FString>& GetTargetDeviceIds(FName InVariant) const override;
	virtual FString GetTargetPlatformName(FName InVariant) const override;
	virtual FName GetTargetPlatformId(FName InVariant) const override;
	virtual FName GetVanillaPlatformId(FName InVariant) const override;
	virtual FText GetPlatformDisplayName(FName InVariant) const override;

	virtual const FString& GetHostName() const override
	{
		return HostName;
	}

	virtual const FString& GetHostUser() const override
	{
		return HostUser;
	}

	virtual const FString& GetMake() const override
	{
		return Make;
	}

	virtual const FString& GetModel() const override
	{
		return Model;
	}

	virtual const FString& GetName() const override
	{
		return Name;
	}

	virtual const FString& GetDeviceUser() const override
	{
		return DeviceUser;
	}

	virtual const FString& GetDeviceUserPassword() const override
	{
		return DeviceUserPassword;
	}

	virtual const FString& GetType() const override
	{
		return Type;
	}

	virtual const FString& GetOSVersion() const override
	{
		return OSVersion;
	}

	virtual FStringView GetArchitecture() const override
	{
		return Architecture;
	}
	
	virtual const FString& GetConnectionType() const override
	{
		return ConnectionType;
	}

	virtual bool HasDeviceId(const FString& InDeviceId) const override;
	virtual bool HasVariant(FName InVariant) const override;
	virtual bool HasTargetPlatform(FName InTargetPlatformId) const override;

	virtual bool IsConnected() const override
	{
		return Connected;
	}

	virtual bool IsAuthorized() const override
	{
		return Authorized;
	}

	virtual bool IsShared() const override
	{
		return Shared;
	}
	
	virtual bool IsSimulated() const override
	{
		return (ConnectionType == TEXT("Simulator"));
	}
	
	virtual bool TerminateLaunchedProcess(FName InVariant, const FString& ProcessIdentifier) override;

	virtual void PowerOff(bool Force) override;
	virtual void PowerOn() override;
	virtual void Reboot() override;

	/** Returns true if this is an aggregate (All_<platform>_devices_on_<host>) proxy, false otherwise */
	virtual bool IsAggregated() const override
	{
		return Aggregated;
	}

protected:

	/** Initializes the message endpoint. */
	void InitializeMessaging();

private:

	/** Holds a flag indicating whether the device is connected. */
	bool Connected;

	/** Holds a flag indicating whether the device is authorized */
	bool Authorized;

	/** Holds the name of the computer that hosts the device. */
	FString HostName;

	/** Holds the name of the user that owns the device. */
	FString HostUser;

	/** The time by which the lack of any ping response is considered a timeout. */
	TOptional<FDateTime> PingTimeout;

	/** Holds the device make. */
	FString Make;

	/** Holds the remote device's message bus address. */
	FMessageAddress MessageAddress;

	/** Holds the local message bus endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Holds the remote device's model. */
	FString Model;

	/** Holds the name of the device. */
	FString Name;

	/** Holds device user. */
	FString DeviceUser;

	/** Holds device user password. */
	FString DeviceUserPassword;

	/** Holds a flag indicating whether the device is being shared with other users. */
	bool Shared;

	/** Holds a flag indicating whether the device supports multi-launch. */
	bool SupportsMultiLaunch;

	/** Holds a flag indicating whether the device can power off. */
	bool SupportsPowerOff;

	/** Holds a flag indicating whether the device can be power on. */
	bool SupportsPowerOn;

	/** Holds a flag indicating whether the device can reboot. */
	bool SupportsReboot;

	/** Holds a flag indicating whether the device's target platform supports variants. */
	bool SupportsVariants;

	/** Holds the device type. */
	FString Type;

	/** Holds the remote device's architecture. */
	FString Architecture;
	
	/** Holds the remote device's OS version. */
	FString OSVersion;

	/** Holds the type of connection. */
	FString ConnectionType;

	/** Holds default variant name. */
	FName DefaultVariant;


	/** Holds a flag indicating whether the proxy is of type "All Devices". */
	bool Aggregated;

	/** Holds data about a device proxy variant. */
	class FTargetDeviceProxyVariant
	{
	public:

		/** Holds a list with the string versions of device ids. */
		TSet<FString> DeviceIDs;

		/** Holds the variant name, this is the the map key as well. */
		FName VariantName;

		/** Platform information. */
		FString TargetPlatformName;
		FName TargetPlatformId;
		FName VanillaPlatformId;
		FText PlatformDisplayName;
	};

	/** Map of all the Variants for this Device. */
	TMap<FName, FTargetDeviceProxyVariant> TargetDeviceVariants;
};
