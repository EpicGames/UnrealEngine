// Copyright Epic Games, Inc. All Rights Reserved.

#include "Proxies/TargetDeviceProxyManager.h"

#include "HAL/PlatformProcess.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"

#include "Proxies/TargetDeviceProxy.h"
#include "TargetDeviceServiceMessages.h"


/** Defines the interval in seconds in which devices are being pinged by the proxy manager. */
#define TARGET_DEVICE_SERVICES_PING_INTERVAL 2.5f

LLM_DEFINE_TAG(TargetDeviceProxyManager);

/* FTargetDeviceProxyManager structors
 *****************************************************************************/

FTargetDeviceProxyManager::FTargetDeviceProxyManager()
{
	MessageEndpoint = FMessageEndpoint::Builder("FTargetDeviceProxyManager")
		.Handling<FTargetDeviceServicePong>(this, &FTargetDeviceProxyManager::HandlePongMessage);

	if (MessageEndpoint.IsValid())
	{
		TickDelegate = FTickerDelegate::CreateRaw(this, &FTargetDeviceProxyManager::HandleTicker);
		TickDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(TickDelegate, TARGET_DEVICE_SERVICES_PING_INTERVAL);

		SendPing();
	}
}


FTargetDeviceProxyManager::~FTargetDeviceProxyManager()
{
	FTSTicker::RemoveTicker(TickDelegateHandle);
	FMessageEndpoint::SafeRelease(MessageEndpoint);
}


/* ITargetDeviceProxyLocator interface
 *****************************************************************************/

TSharedPtr<ITargetDeviceProxy> FTargetDeviceProxyManager::FindProxy(const FString& Name) 
{
	return Proxies.FindRef(Name);
}


TSharedRef<ITargetDeviceProxy> FTargetDeviceProxyManager::FindOrAddProxy(const FString& Name)
{
	TSharedPtr<FTargetDeviceProxy>& Proxy = Proxies.FindOrAdd(Name);

	if (!Proxy.IsValid())
	{
		Proxy = MakeShareable(new FTargetDeviceProxy(Name));

		ProxyAddedDelegate.Broadcast(Proxy.ToSharedRef());
	}

	return Proxy.ToSharedRef();
}


TSharedPtr<ITargetDeviceProxy> FTargetDeviceProxyManager::FindProxyDeviceForTargetDevice(const FString& DeviceId)
{
	for (TMap<FString, TSharedPtr<FTargetDeviceProxy> >::TConstIterator ItProxies(Proxies); ItProxies; ++ItProxies)
	{
		const TSharedPtr<FTargetDeviceProxy>& Proxy = ItProxies.Value();

		if (Proxy->HasDeviceId(DeviceId))
		{
			return Proxy;
		}
	}

	return TSharedPtr<ITargetDeviceProxy>();
}


void FTargetDeviceProxyManager::GetProxies(FName TargetPlatformName, bool IncludeUnshared, TArray<TSharedPtr<ITargetDeviceProxy>>& OutProxies)
{
	GetProxyList(TargetPlatformName, IncludeUnshared, false, OutProxies);
}


// the proxy list include aggregate (All_<platform>_devices_on_<host>) proxies
void FTargetDeviceProxyManager::GetAllProxies(FName TargetPlatformName, TArray<TSharedPtr<ITargetDeviceProxy>>& OutProxies)
{
	GetProxyList(TargetPlatformName, false, true, OutProxies);
}


/** Gets a filtered list of proxies created by the device discovery routine */
void FTargetDeviceProxyManager::GetProxyList(FName TargetPlatformName, bool IncludeUnshared, bool bIncludeAggregate, TArray<TSharedPtr<ITargetDeviceProxy>>& OutProxies)
{
	OutProxies.Reset();

	for (TMap<FString, TSharedPtr<FTargetDeviceProxy> >::TConstIterator It(Proxies); It; ++It)
	{
		const TSharedPtr<FTargetDeviceProxy>& Proxy = It.Value();

		if ((IncludeUnshared || Proxy->IsShared()) || (Proxy->GetHostUser() == FPlatformProcess::UserName(false)))
		{
			if (TargetPlatformName == NAME_None || Proxy->HasTargetPlatform(TargetPlatformName))
			{
				if (bIncludeAggregate || !Proxy->IsAggregated())
				{
					OutProxies.Add(Proxy);
				}
			}		
		}
	}
}

/* FTargetDeviceProxyManager implementation
 *****************************************************************************/

void FTargetDeviceProxyManager::RemoveDeadProxies()
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FTargetDeviceProxyManager_RemoveDeadProxies);

	FDateTime CurrentTime = FDateTime::UtcNow();

	for (auto ProxyIter = Proxies.CreateIterator(); ProxyIter; ++ProxyIter)
	{
		if (ProxyIter.Value()->HasTimedOut(CurrentTime))
		{
			TSharedPtr<ITargetDeviceProxy> RemovedProxy = ProxyIter.Value();
			ProxyIter.RemoveCurrent();
			ProxyRemovedDelegate.Broadcast(RemovedProxy.ToSharedRef());
		}
	}
}


void FTargetDeviceProxyManager::SendPing()
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FTargetDeviceProxyManager_SendPing);

	if (MessageEndpoint.IsValid())
	{
		// Add a timeout to all known devices
		const FDateTime Timeout = FDateTime::UtcNow() + FTimespan:: FromSeconds(3 * TARGET_DEVICE_SERVICES_PING_INTERVAL);
		for (const TPair<FString, TSharedPtr<FTargetDeviceProxy>>& Proxy : Proxies)
		{
			Proxy.Value->AddTimeout(Timeout);
		}

		// send to only this process, as we no longer want to send out pings to the local network and get another machine's list of devices
		MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FTargetDeviceServicePing>(FPlatformProcess::UserName(false)), EMessageScope::Process);
	}
}


/* FTargetDeviceProxyManager callbacks
 *****************************************************************************/

void FTargetDeviceProxyManager::HandlePongMessage(const FTargetDeviceServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	// Another HACK: Ignore devices from other machines. See FTargetDeviceService::HandleClaimDeniedMessage()
	if (Message.HostName != FPlatformProcess::ComputerName())
 	{
 		return;
 	}

	LLM_SCOPE_BYTAG(TargetDeviceProxyManager);

	AddProxyFromPongMessage(Message, Context, false);

	if (Message.Aggregated)
	{
		// add the device to the aggregate (All_<platform>_devices_on_<host>) proxy
		// create the aggregate proxy if it wasn't created already by a previous messga
		AddProxyFromPongMessage(Message, Context, true);
	}
}

/** Add or update the proxy from the FTargetDeviceServicePong message */
void FTargetDeviceProxyManager::AddProxyFromPongMessage(const FTargetDeviceServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, bool InIsAggregated)
{
	FString ProxyName = InIsAggregated ? Message.AllDevicesName : Message.Name;
	TSharedPtr<FTargetDeviceProxy>& Proxy = Proxies.FindOrAdd(ProxyName);

	if (!Proxy.IsValid())
	{
		Proxy = MakeShareable(new FTargetDeviceProxy(ProxyName, Message, Context, InIsAggregated));
		ProxyAddedDelegate.Broadcast(Proxy.ToSharedRef());
	}
	else
	{
		Proxy->UpdateFromMessage(Message, Context);
	}
}


bool FTargetDeviceProxyManager::HandleTicker(float DeltaTime)
{
	LLM_SCOPE_BYTAG(TargetDeviceProxyManager);

	RemoveDeadProxies();
	SendPing();

	return true;
}
