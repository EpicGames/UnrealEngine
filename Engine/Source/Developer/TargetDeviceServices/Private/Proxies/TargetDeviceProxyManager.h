// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

#include "ITargetDeviceProxyManager.h"

class FMessageEndpoint;
class FTargetDeviceProxy;
class IMessageContext;

struct FTargetDeviceServicePong;


/**
 * Implements a class which locates devices based on criteria for use in the Launcher.
 */
class FTargetDeviceProxyManager
	: public ITargetDeviceProxyManager
{
public:

	/** Default constructor. */
	FTargetDeviceProxyManager();

	/** Virtual destructor. */
	virtual ~FTargetDeviceProxyManager();

public:

	//~ ITargetDeviceProxyLocator interface

	virtual TSharedRef<ITargetDeviceProxy> FindOrAddProxy(const FString& Name) override;
	virtual TSharedPtr<ITargetDeviceProxy> FindProxy(const FString& Name) override;
	virtual TSharedPtr<ITargetDeviceProxy> FindProxyDeviceForTargetDevice(const FString& DeviceId) override;
	virtual void GetProxies(FName TargetPlatformName, bool IncludeUnshared, TArray<TSharedPtr<ITargetDeviceProxy>>& OutProxies) override;
	
	// the proxy list include aggregate (All_<platform>_devices_on_<host>) proxies
	virtual void GetAllProxies(FName TargetPlatformName, TArray<TSharedPtr<ITargetDeviceProxy>>& OutProxies) override;

	DECLARE_DERIVED_EVENT(FTargetDeviceProxyManager, ITargetDeviceProxyManager::FOnTargetDeviceProxyAdded, FOnTargetDeviceProxyAdded);
	virtual FOnTargetDeviceProxyAdded& OnProxyAdded() override
	{
		return ProxyAddedDelegate;
	}

	DECLARE_DERIVED_EVENT(FTargetDeviceProxyManager, ITargetDeviceProxyManager::FOnTargetDeviceProxyRemoved, FOnTargetDeviceProxyRemoved);
	virtual FOnTargetDeviceProxyRemoved& OnProxyRemoved() override
	{
		return ProxyRemovedDelegate;
	}

protected:

	/** Removes all target device proxies that timed out. */
	void RemoveDeadProxies();

	/** Pings all target devices on the network. */
	void SendPing();

private:
	
	/** Handles FTargetDeviceServicePong messages. */
	void HandlePongMessage(const FTargetDeviceServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Add or update the proxy from the FTargetDeviceServicePong message */
	void AddProxyFromPongMessage(const FTargetDeviceServicePong& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, bool InIsAggregated);

	/** Handles ticks from the ticker. */
	bool HandleTicker(float DeltaTime);

	/**
	* Gets a filtered list of proxies created by the device discovery routine
	*
	* @param PlatformName The the name of the target platform to get proxies for (or empty string for all proxies).
	* @param IncludeUnshared Will hold the list of devices found by the locator.
	* @param IncludeAggregate Will include the "All devices" entries.
	* @param OutProxies Will hold the list of devices found by the locator.
	* @see FindOrAddProxy, FindProxy, FindProxyDeviceForTargetDevice
	*/
	void GetProxyList(FName TargetPlatformName, bool IncludeUnshared, bool bIncludeAggregate, TArray<TSharedPtr<ITargetDeviceProxy>>& OutProxies);
private:

	/** Holds the message endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Holds the collection of proxies. */
	TMap<FString, TSharedPtr<FTargetDeviceProxy>> Proxies;

private:

	/** Holds a delegate that is invoked when a target device proxy has been added. */
	FOnTargetDeviceProxyAdded ProxyAddedDelegate;

	/** Holds a delegate that is invoked when a target device proxy has been removed. */
	FOnTargetDeviceProxyRemoved ProxyRemovedDelegate;

	/** Holds a delegate to be invoked when the widget ticks. */
	FTickerDelegate TickDelegate;

	/** Handle to the registered TickDelegate. */
	FTSTicker::FDelegateHandle TickDelegateHandle;
};
