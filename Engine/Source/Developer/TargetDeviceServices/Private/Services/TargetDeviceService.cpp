// Copyright Epic Games, Inc. All Rights Reserved.

#include "Services/TargetDeviceService.h"

#include "HAL/PlatformProcess.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "IMessageBus.h"
#include "Interfaces/ITargetDevice.h"
#include "Interfaces/ITargetPlatform.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "PlatformInfo.h"
#include "Serialization/Archive.h"
#include "Interfaces/ITargetPlatformSettings.h"
#include "Interfaces/ITargetPlatformControls.h"
#include "TargetDeviceServiceMessages.h"

LLM_DECLARE_TAG(TargetDeviceProxyManager);


/* Local helpers
 *****************************************************************************/

struct FVariantSortCallback
{
	FORCEINLINE bool operator()(const ITargetDeviceWeakPtr& A, const ITargetDeviceWeakPtr& B) const
	{
		ITargetDevicePtr APtr = A.Pin();
		ITargetDevicePtr BPtr = B.Pin();

		return APtr->GetPlatformControls().GetVariantPriority() > BPtr->GetPlatformControls().GetVariantPriority();
	}
};


/* FTargetDeviceService structors
 *****************************************************************************/

FTargetDeviceService::FTargetDeviceService(const FString& InDeviceName, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus)
	: DeviceName(InDeviceName)
	, Running(false)
	, Shared(false)
{
	// initialize messaging
	MessageEndpoint = FMessageEndpoint::Builder(FName(*FString::Printf(TEXT("FTargetDeviceService (%s)"), *DeviceName)), InMessageBus)
		.Handling<FTargetDeviceClaimDenied>(this, &FTargetDeviceService::HandleClaimDeniedMessage)
		.Handling<FTargetDeviceClaimed>(this, &FTargetDeviceService::HandleClaimedMessage)
		.Handling<FTargetDeviceServiceTerminateLaunchedProcess>(this, &FTargetDeviceService::HandleTerminateLaunchedProcessMessage)
		.Handling<FTargetDeviceServicePing>(this, &FTargetDeviceService::HandlePingMessage)
		.Handling<FTargetDeviceServicePowerOff>(this, &FTargetDeviceService::HandlePowerOffMessage)
		.Handling<FTargetDeviceServicePowerOn>(this, &FTargetDeviceService::HandlePowerOnMessage)
		.Handling<FTargetDeviceServiceReboot>(this, &FTargetDeviceService::HandleRebootMessage)
		.Handling<FTargetDeviceUnclaimed>(this, &FTargetDeviceService::HandleUnclaimedMessage);

	if (MessageEndpoint.IsValid())
	{
		MessageEndpoint->Subscribe<FTargetDeviceClaimed>();
		MessageEndpoint->Subscribe<FTargetDeviceUnclaimed>();
		MessageEndpoint->Subscribe<FTargetDeviceServicePing>();
	}
}


FTargetDeviceService::~FTargetDeviceService()
{
	Stop();
	FMessageEndpoint::SafeRelease(MessageEndpoint);
}


/* ITargetDeviceService interface
 *****************************************************************************/

void FTargetDeviceService::AddTargetDevice(TSharedPtr<ITargetDevice, ESPMode::ThreadSafe> InDevice)
{
	if (!InDevice.IsValid())
	{
		return;
	}

	FName Variant = FName(InDevice->GetPlatformControls().PlatformName().GetCharArray().GetData());

	if (DevicePlatformName == NAME_None)
	{
		// If this seems nasty your right!
		// This is just one more nastiness in this class due to the fact that we intend to refactor the target platform stuff as a separate task.
		const PlatformInfo::FTargetPlatformInfo& Info = InDevice->GetPlatformControls().GetTargetPlatformInfo();
		const PlatformInfo::FTargetPlatformInfo* VanillaInfo = Info.VanillaInfo;

		DevicePlatformName = Info.Name;
		DevicePlatformDisplayName = VanillaInfo->DisplayName.ToString();
		
		// Sigh the hacks... Should be able to remove if platform info gets cleaned up.... Windows doesn't have a reasonable vanilla platform.
		const FString VariableSplit(TEXT("("));
		FString Full = VanillaInfo->DisplayName.ToString();
		FString Left;
		FString Right;
		bool bSplit = Full.Split(VariableSplit, &Left, &Right);
		DevicePlatformDisplayName = bSplit ? Left.TrimStart() : Full;
	}

	// double add, which due to the async nature of some device discovery can't be easily avoided.
	if (!(TargetDevicePtrs.FindRef(Variant).IsValid()))
	{
		TargetDevicePtrs.Add(Variant, InDevice);

		// sort and choose cache the default
		TargetDevicePtrs.ValueSort(FVariantSortCallback());
		auto DeviceIterator = TargetDevicePtrs.CreateIterator();

		if (DeviceIterator)
		{
			DefaultDevicePtr = (*DeviceIterator).Value;
		}
		else
		{
			DefaultDevicePtr = nullptr;
		}
	}
}


bool FTargetDeviceService::CanStart(FName InFlavor) const
{
	ITargetDevicePtr TargetDevice = GetDevice(InFlavor);

	if (TargetDevice.IsValid())
	{
		return TargetDevice->IsConnected();
	}

	return false;
}


const FString& FTargetDeviceService::GetClaimHost()
{
	return ClaimHost;
}


const FString& FTargetDeviceService::GetClaimUser()
{
	return ClaimUser;
}


ITargetDevicePtr FTargetDeviceService::GetDevice(FName InVariant) const
{
	ITargetDevicePtr TargetDevice;

	if (InVariant == NAME_None)
	{
		TargetDevice = DefaultDevicePtr.Pin();
	}
	else
	{
		const ITargetDeviceWeakPtr * WeakTargetDevicePtr = TargetDevicePtrs.Find(InVariant);

		if (WeakTargetDevicePtr != nullptr)
		{
			TargetDevice = WeakTargetDevicePtr->Pin();
		}
	}

	return TargetDevice;
}


FString FTargetDeviceService::GetDeviceName() const
{
	return DeviceName;
}


FName FTargetDeviceService::GetDevicePlatformName() const
{
	return DevicePlatformName;
}


FString FTargetDeviceService::GetDevicePlatformDisplayName() const
{
	return DevicePlatformDisplayName;
}


bool FTargetDeviceService::IsRunning() const
{
	return Running;
}


bool FTargetDeviceService::IsShared() const
{
	return (Running && Shared);
}


int32 FTargetDeviceService::NumTargetDevices()
{
	return TargetDevicePtrs.Num();
}


void FTargetDeviceService::RemoveTargetDevice(TSharedPtr<ITargetDevice, ESPMode::ThreadSafe> InDevice)
{
	if (!InDevice.IsValid())
	{
		return;
	}

	FName Variant = FName(InDevice->GetPlatformControls().PlatformName().GetCharArray().GetData());

	TargetDevicePtrs.Remove(Variant);

	// Cache the default
	auto DeviceIterator = TargetDevicePtrs.CreateIterator();

	if (DeviceIterator)
	{
		DefaultDevicePtr = (*DeviceIterator).Value;
	}
	else
	{
		DefaultDevicePtr = nullptr;
	}
}


void FTargetDeviceService::SetShared(bool InShared)
{
	Shared = InShared;
}


bool FTargetDeviceService::Start()
{
	if (!Running && MessageEndpoint.IsValid())
	{
		// notify other services
		ClaimAddress = MessageEndpoint->GetAddress();
		ClaimHost = FPlatformProcess::ComputerName();
		ClaimUser = FPlatformProcess::UserName(false);

		MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FTargetDeviceClaimed>(DeviceName, ClaimHost, ClaimUser));

		Running = true;
	}
		
	return true;
}


void FTargetDeviceService::Stop()
{
	if (Running)
	{
		// message is going to be deleted by FMemory::Free() (see FMessageContext destructor), so allocate it with Malloc
		MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<FTargetDeviceUnclaimed>(DeviceName, FPlatformProcess::ComputerName(), FPlatformProcess::UserName(false)));
		FPlatformProcess::SleepNoStats(0.01f);

		// Only stop the device if we care about device claiming
		bool bDisableDeviceClaiming = false;
		if(!GConfig->GetBool(TEXT("/Script/Engine.Engine"), TEXT("DisableDeviceClaiming"), bDisableDeviceClaiming, GEngineIni) || !bDisableDeviceClaiming)
		{
			Running = false;
		}
	}
}


/* FTargetDeviceService implementation
 *****************************************************************************/

bool FTargetDeviceService::StoreDeployedFile(FArchive* FileReader, const FString& TargetFileName) const
{
	if (FileReader == nullptr)
	{
		return false;
	}

	// create target file
	FArchive* FileWriter = IFileManager::Get().CreateFileWriter(*TargetFileName);

	if (FileWriter == nullptr)
	{
		return false;
	}

	FileReader->Seek(0);

	// copy file contents
	int64 BytesRemaining = FileReader->TotalSize();
	int64 BufferSize = 128 * 1024;

	if (BytesRemaining < BufferSize)
	{
		BufferSize = BytesRemaining;
	}

	void* Buffer = FMemory::Malloc(BufferSize);

	while (BytesRemaining > 0)
	{
		FileReader->Serialize(Buffer, BufferSize);
		FileWriter->Serialize(Buffer, BufferSize);

		BytesRemaining -= BufferSize;

		if (BytesRemaining < BufferSize)
		{
			BufferSize = BytesRemaining;
		}
	}

	// clean up
	FMemory::Free(Buffer);
	delete FileWriter;

	return true;
}


/* FTargetDeviceService callbacks
 *****************************************************************************/

void FTargetDeviceService::HandleClaimDeniedMessage(const FTargetDeviceClaimDenied& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
// HACK: Disabling claim denied message. Allows the editor to always claim a device and should prevent cases where instances of the
// editor running on other machines can steal a device from us - which is undesirable on some platforms.
// Also see FTargetDeviceProxyManager::HandlePongMessage()
#if 0
 	if (Running && (Message.DeviceName == DeviceName))
 	{
 		Stop();
 
 		ClaimAddress = Context->GetSender();
 		ClaimHost = Message.HostName;
 		ClaimUser = Message.HostUser;
 	}
#endif
}


void FTargetDeviceService::HandleClaimedMessage(const FTargetDeviceClaimed& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (Message.DeviceName != DeviceName)
	{
		return;
	}

	if (Running)
	{
		if (Context->GetSender() != MessageEndpoint->GetAddress())
		{
			MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FTargetDeviceClaimDenied>(DeviceName, FPlatformProcess::ComputerName(), FPlatformProcess::UserName(false)), Context->GetSender());
		}
	}
	else
	{
		ClaimAddress = Context->GetSender();
		ClaimHost = Message.HostName;
		ClaimUser = Message.HostUser;
	}
}


void FTargetDeviceService::HandleUnclaimedMessage(const FTargetDeviceUnclaimed& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (Message.DeviceName == DeviceName)
	{
		if (Context->GetSender() == ClaimAddress)
		{
			ClaimAddress.Invalidate();
			ClaimHost.Empty();
			ClaimUser.Empty();
		}
	}
}


void FTargetDeviceService::HandleTerminateLaunchedProcessMessage(const FTargetDeviceServiceTerminateLaunchedProcess& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (!Running)
	{
		return;
	}

	ITargetDevicePtr TargetDevice = GetDevice(Message.Variant);

	if (TargetDevice.IsValid())
	{
		TargetDevice->TerminateLaunchedProcess(Message.AppID);
	}
}

void FTargetDeviceService::HandlePingMessage(const FTargetDeviceServicePing& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
{
	if (!Running)
	{
		return;
	}

	if (!Shared && (InMessage.HostUser != FPlatformProcess::UserName(false)))
	{
		return;
	}

	LLM_SCOPE_BYTAG(TargetDeviceProxyManager);

	ITargetDevicePtr DefaultDevice = GetDevice(); // Default Device is needed here!

	if (DefaultDevice.IsValid())
	{
		const FString& PlatformName = DefaultDevice->GetPlatformControls().PlatformName();
		const PlatformInfo::FTargetPlatformInfo* VanillaInfo = DefaultDevice->GetPlatformControls().GetTargetPlatformInfo().VanillaInfo;

		FTargetDeviceServicePong* Message = FMessageEndpoint::MakeMessage<FTargetDeviceServicePong>();

		Message->Name = DefaultDevice->GetName();
		Message->Type = TargetDeviceTypes::ToString(DefaultDevice->GetDeviceType());
		Message->HostName = FPlatformProcess::ComputerName();
		Message->HostUser = FPlatformProcess::UserName(false);
		Message->Connected = DefaultDevice->IsConnected();
		Message->ConnectionType = TargetDeviceConnectionTypes::ToString(DefaultDevice->GetDeviceConnectionType());
		Message->Authorized = DefaultDevice->IsAuthorized();
		Message->Make = TEXT("@todo");
		Message->Model = DefaultDevice->GetModelId();
		Message->OSVersion = DefaultDevice->GetOSVersion();
		Message->Architecture = DefaultDevice->GetArchitecture();
		DefaultDevice->GetUserCredentials(Message->DeviceUser, Message->DeviceUserPassword);
		Message->Shared = Shared;
		Message->SupportsMultiLaunch = DefaultDevice->SupportsFeature(ETargetDeviceFeatures::MultiLaunch);
		Message->SupportsPowerOff = DefaultDevice->SupportsFeature(ETargetDeviceFeatures::PowerOff);
		Message->SupportsPowerOn = DefaultDevice->SupportsFeature(ETargetDeviceFeatures::PowerOn);
		Message->SupportsReboot = DefaultDevice->SupportsFeature(ETargetDeviceFeatures::Reboot);
		Message->SupportsVariants = DefaultDevice->GetPlatformControls().SupportsVariants();
		Message->DefaultVariant = FName(DefaultDevice->GetPlatformControls().PlatformName().GetCharArray().GetData());

		// Check if we should also create an aggregate (All_<platform>_devices_on_<host>) proxy
		Message->Aggregated = DefaultDevice->IsPlatformAggregated();
		Message->AllDevicesName = DefaultDevice->GetAllDevicesName().IsEmpty() ? VanillaInfo->Name.ToString() : DefaultDevice->GetAllDevicesName();
		Message->AllDevicesDefaultVariant = DefaultDevice->GetAllDevicesDefaultVariant().IsNone() ? Message->DefaultVariant : DefaultDevice->GetAllDevicesDefaultVariant();

		// Add the data for all the flavors
		Message->Variants.SetNumZeroed(TargetDevicePtrs.Num());

		int Index = 0;
		for (auto TargetDeviceIt = TargetDevicePtrs.CreateIterator(); TargetDeviceIt; ++TargetDeviceIt, ++Index)
		{
			const ITargetDevicePtr& TargetDevice = TargetDeviceIt.Value().Pin();
			const PlatformInfo::FTargetPlatformInfo& Info = TargetDevice->GetPlatformControls().GetTargetPlatformInfo();

			FTargetDeviceVariant& Variant = Message->Variants[Index];

			Variant.DeviceID = TargetDevice->GetId().ToString();
			Variant.VariantName = TargetDeviceIt.Key();
			Variant.TargetPlatformName = TargetDevice->GetPlatformControls().PlatformName();
			Variant.TargetPlatformId = Info.Name;
			Variant.VanillaPlatformId = Info.VanillaInfo->Name;
			Variant.PlatformDisplayName = Info.DisplayName.ToString();
		}

		MessageEndpoint->Send(Message, Context->GetSender());
	}
}


void FTargetDeviceService::HandlePowerOffMessage( const FTargetDeviceServicePowerOff& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if (!Running)
	{
		return;
	}

	ITargetDevicePtr TargetDevice = GetDevice(); // Any Device is fine here

	if (TargetDevice.IsValid())
	{
		TargetDevice->PowerOff(Message.Force);
	}
}


void FTargetDeviceService::HandlePowerOnMessage( const FTargetDeviceServicePowerOn& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if (!Running)
	{
		return;
	}

	ITargetDevicePtr TargetDevice = GetDevice(); // Any Device is fine here

	if (TargetDevice.IsValid())
	{
		TargetDevice->PowerOn();
	}
}


void FTargetDeviceService::HandleRebootMessage( const FTargetDeviceServiceReboot& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context )
{
	if (!Running)
	{
		return;
	}

	ITargetDevicePtr TargetDevice = GetDevice(); // Any Device is fine here

	if (TargetDevice.IsValid())
	{
		TargetDevice->Reboot();
	}
}
