// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "IMessageContext.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#include "ITargetDeviceService.h"

class FArchive;
class FMessageEndpoint;
class IMessageBus;
class ITargetDevice;

struct FTargetDeviceClaimDenied;
struct FTargetDeviceClaimed;
struct FTargetDeviceServiceDeployCommit;
struct FTargetDeviceServiceDeployFile;
struct FTargetDeviceServiceLaunchApp;
struct FTargetDeviceServiceTerminateLaunchedProcess;
struct FTargetDeviceServicePing;
struct FTargetDeviceServicePowerOff;
struct FTargetDeviceServicePowerOn;
struct FTargetDeviceServiceReboot;
struct FTargetDeviceServiceRunExecutable;
struct FTargetDeviceUnclaimed;


/**
 * Implements remote services for a specific target device.
 */
class FTargetDeviceService
	: public ITargetDeviceService
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InDeviceName The name of the device to expose.
	 * @param InMessageBus The message bus to listen on for clients.
	 */
	FTargetDeviceService(const FString& InDeviceName, const TSharedRef<IMessageBus, ESPMode::ThreadSafe>& InMessageBus);

	/** Destructor. */
	~FTargetDeviceService();

public:

	//~ ITargetDeviceService interface

	virtual void AddTargetDevice(TSharedPtr<ITargetDevice, ESPMode::ThreadSafe> InDevice) override;
	virtual bool CanStart(FName InFlavor = NAME_None) const override;
	virtual const FString& GetClaimHost() override;
	virtual const FString& GetClaimUser() override;
	virtual ITargetDevicePtr GetDevice(FName InVariant = NAME_None) const override;
	virtual FString GetDeviceName() const override;
	virtual FName GetDevicePlatformName() const override;
	virtual FString GetDevicePlatformDisplayName() const override;
	virtual bool IsRunning() const override;
	virtual bool IsShared() const override;
	virtual int32 NumTargetDevices() override;
	virtual void RemoveTargetDevice(TSharedPtr<ITargetDevice, ESPMode::ThreadSafe> InDevice) override;
	virtual void SetShared(bool InShared) override;
	virtual bool Start() override;
	virtual void Stop() override;

protected:

	/**
	 * Stores the specified file to deploy.
	 *
	 * @param FileReader The archive reader providing access to the file data.
	 * @param TargetFileName The desired name of the file on the target device.
	 */
	bool StoreDeployedFile(FArchive* FileReader, const FString& TargetFileName) const;

private:

	/** Callback for FTargetDeviceClaimDenied messages. */
	void HandleClaimDeniedMessage(const FTargetDeviceClaimDenied& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Callback for FTargetDeviceClaimDenied messages. */
	void HandleClaimedMessage( const FTargetDeviceClaimed& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context );

	/** Callback for FTargetDeviceClaimDropped messages. */
	void HandleUnclaimedMessage(const FTargetDeviceUnclaimed& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Callback for FTargetDeviceServiceTerminateLaunchedProcess messages. */
	void HandleTerminateLaunchedProcessMessage(const FTargetDeviceServiceTerminateLaunchedProcess& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Callback for FTargetDeviceServicePing messages. */
	void HandlePingMessage(const FTargetDeviceServicePing& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	 /** Callback for FTargetDeviceServicePowerOff messages. */
	void HandlePowerOffMessage(const FTargetDeviceServicePowerOff& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Callback for FTargetDeviceServicePowerOn messages. */
	void HandlePowerOnMessage(const FTargetDeviceServicePowerOn& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

	/** Callback for FTargetDeviceServiceReboot messages. */
	void HandleRebootMessage(const FTargetDeviceServiceReboot& Message, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context);

private:

	/** Default target device used when no flavor is specified. */
	TWeakPtr<ITargetDevice, ESPMode::ThreadSafe> DefaultDevicePtr;

	/** Caches the name of the device name that this services exposes. */
	FString DeviceName;

	/** Caches the platform name of the device name that this services exposes. */
	FName DevicePlatformName;

	/** Caches the platform name of the device name that this services exposes. */
	FString DevicePlatformDisplayName;

	/** Holds the name of the host that has a claim on the device. */
	FString ClaimHost;

	/** Holds the message address of the target device service that has a claim on the device. */
	FMessageAddress ClaimAddress;

	/** Holds the name of the user that has a claim on the device. */
	FString ClaimUser;

	/** Holds the message endpoint. */
	TSharedPtr<FMessageEndpoint, ESPMode::ThreadSafe> MessageEndpoint;

	/** Holds a flag indicating whether this service is currently running. */
	bool Running;

	/** Holds a flag indicating whether the device is shared with other users. */
	bool Shared;
	
	/** Map of all the Flavors for this Service */
	TMap<FName, TWeakPtr<ITargetDevice, ESPMode::ThreadSafe>> TargetDevicePtrs;
};
