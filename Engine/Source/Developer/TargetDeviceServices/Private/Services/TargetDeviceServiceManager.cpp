// Copyright Epic Games, Inc. All Rights Reserved.

#include "Services/TargetDeviceServiceManager.h"
#include "TargetDeviceServicesPrivate.h"

#include "IMessageBus.h"
#include "Misc/ScopeLock.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "IMessagingModule.h"

#include "ITargetDeviceService.h"
#include "Services/TargetDeviceService.h"



/* FTargetDeviceServiceManager structors
 *****************************************************************************/

FTargetDeviceServiceManager::FTargetDeviceServiceManager()
{
	TSharedPtr<IMessageBus, ESPMode::ThreadSafe> MessageBus = IMessagingModule::Get().GetDefaultBus();

	if (MessageBus.IsValid())
	{
		MessageBus->OnShutdown().AddRaw(this, &FTargetDeviceServiceManager::HandleMessageBusShutdown);
		MessageBusPtr = MessageBus;
	}

	LoadSettings();
	InitializeTargetPlatforms();
}


FTargetDeviceServiceManager::~FTargetDeviceServiceManager()
{
	ShutdownTargetPlatforms();

	FScopeLock Lock(&CriticalSection);
	{
		SaveSettings();
	}

	auto MessageBus = MessageBusPtr.Pin();

	if (MessageBus.IsValid())
	{
		MessageBus->OnShutdown().RemoveAll(this);
	}
}


/* ITargetDeviceServiceManager interface
 *****************************************************************************/

bool FTargetDeviceServiceManager::AddStartupService(const FString& DeviceName)
{
	FScopeLock Lock(&CriticalSection);
	{
		StartupServices.Add(DeviceName, false);

		return AddService(DeviceName).IsValid();
	}
}


int32 FTargetDeviceServiceManager::GetServices( TArray<ITargetDeviceServicePtr>& OutServices )
{
	FScopeLock Lock(&CriticalSection);
	{
		DeviceServices.GenerateValueArray(OutServices);
	}

	return OutServices.Num();
}


void FTargetDeviceServiceManager::RemoveStartupService(const FString& DeviceName)
{
	FScopeLock Lock(&CriticalSection);
	{
		if (StartupServices.Remove(DeviceName) > 0)
		{
			RemoveService(DeviceName);
		}
	}
}


/* FTargetDeviceServiceManager implementation
 *****************************************************************************/

ITargetDeviceServicePtr FTargetDeviceServiceManager::AddService(const FString& DeviceName)
{
	auto MessageBus = MessageBusPtr.Pin();

	if (!MessageBus.IsValid())
	{
		return nullptr;
	}

	ITargetDeviceServicePtr DeviceService = DeviceServices.FindRef(DeviceName);

	// create service if needed
	if (!DeviceService.IsValid())
	{
		DeviceService = MakeShareable(new FTargetDeviceService(DeviceName, MessageBus.ToSharedRef()));
		DeviceServices.Add(DeviceName, DeviceService);

		ServiceAddedDelegate.Broadcast(DeviceService.ToSharedRef());
	}

	// share service if desired
	const bool* Shared = StartupServices.Find(DeviceName);
	if (Shared != nullptr)
	{
		DeviceService->SetShared(*Shared);
		DeviceService->Start();
	}

	return DeviceService;
}

bool FTargetDeviceServiceManager::AddTargetDevice(ITargetDevicePtr InDevice)
{
	FScopeLock Lock(&CriticalSection);
	
	auto MessageBus = MessageBusPtr.Pin();
	
	if (!MessageBus.IsValid())
	{
		return false;
	}

	FString DeviceName = InDevice->GetName();
	ITargetDeviceServicePtr DeviceService = AddService(DeviceName);

	if (DeviceService.IsValid())
	{
		DeviceService->AddTargetDevice(InDevice);

		if (InDevice.IsValid() && InDevice->IsDefault())
		{
			DeviceService->Start();
		}
	}

	return true;
}


void FTargetDeviceServiceManager::InitializeTargetPlatforms()
{
	ITargetPlatformControls::OnDeviceDiscovered().AddRaw(this, &FTargetDeviceServiceManager::HandleTargetPlatformDeviceDiscovered);
	ITargetPlatformControls::OnDeviceLost().AddRaw(this, &FTargetDeviceServiceManager::HandleTargetPlatformDeviceLost);
	
	for (ITargetPlatform* Platform : GetTargetPlatformManager()->GetTargetPlatforms())
	{
		// add services for existing devices
		TArray<ITargetDevicePtr> Devices;
		Platform->GetAllDevices(Devices);

		for (int32 DeviceIndex = 0; DeviceIndex < Devices.Num(); ++DeviceIndex)
		{
			ITargetDevicePtr& Device = Devices[DeviceIndex];

			if (Device.IsValid())
			{
				AddTargetDevice(Device);
			}
		}
	}
}


void FTargetDeviceServiceManager::LoadSettings()
{
	if (GConfig == nullptr)
	{
		return;
	}

	const FConfigSection* OwnedDevices = GConfig->GetSection(TEXT("TargetDeviceServices"), false, GEngineIni);
	
	if (OwnedDevices == nullptr)
	{
		return;
	}

	// for each entry in the INI file...
	for (FConfigSection::TConstIterator It(*OwnedDevices); It; ++It)
	{
		if (It.Key() != TEXT("StartupServices"))
		{
			continue;
		}

		const FString& ServiceString = It.Value().GetValue();

		// ... parse device identifier...
		FString DeviceName;
		
		if (!FParse::Value(*ServiceString, TEXT("DeviceName="), DeviceName))
		{
			UE_LOGF(TargetDeviceServicesLog, Warning, "[TargetDeviceServices] failed to parse DeviceName in configuration setting: StartupServices=%ls", *ServiceString);

			continue;
		}

		if (DeviceServices.Contains(DeviceName))
		{
			UE_LOGF(TargetDeviceServicesLog, Warning, "[TargetDeviceServices] duplicate entry for : StartupServices=%ls", *ServiceString);

			continue;
		}

		// ... parse sharing state...
		bool Shared;

		if (FParse::Bool(*ServiceString, TEXT("Shared="), Shared))
		{
			Shared = false;
		}

		StartupServices.Add(DeviceName, Shared);

		// ... create and start device service
		ITargetDeviceServicePtr DeviceService;

		if (!AddService(DeviceName).IsValid())
		{
			UE_LOGF(TargetDeviceServicesLog, Warning, "[TargetDeviceServices] failed to create service for: StartupServices=%ls", *ServiceString);
		}
	}
}


void FTargetDeviceServiceManager::RemoveService(const FString& DeviceName)
{
	ITargetDeviceServicePtr DeviceService = DeviceServices.FindRef(DeviceName);

	if (!DeviceService.IsValid())
	{
		return;
	}

	DeviceService->Stop();

	// only truly remove if not startup device or physical device not available
	if (!StartupServices.Contains(DeviceName) && !DeviceService->GetDevice().IsValid())
	{
		DeviceServices.Remove(DeviceName);
		ServiceRemovedDelegate.Broadcast(DeviceService.ToSharedRef());
	}
}

void FTargetDeviceServiceManager::RemoveTargetDevice(ITargetDevicePtr InDevice)
{
	FScopeLock Lock(&CriticalSection);
	ITargetDeviceServicePtr DeviceService = DeviceServices.FindRef(InDevice->GetName());

	if (!DeviceService.IsValid())
	{
		return;
	}

	DeviceService->RemoveTargetDevice(InDevice);

	if (DeviceService->NumTargetDevices() <= 0)
	{
		RemoveService(InDevice->GetName());
	}

	return;
}


void FTargetDeviceServiceManager::SaveSettings()
{
	if (GConfig == nullptr)
	{
		return;
	}

	GConfig->EmptySection(TEXT("TargetDeviceServices"), GEngineIni);

	TArray<FString> ServiceStrings;

	// for each device service...
	for (TMap<FString, ITargetDeviceServicePtr>::TConstIterator It(DeviceServices); It; ++It)
	{
		const FString& DeviceName= It.Key();
		const ITargetDeviceServicePtr& DeviceService = It.Value();

		// ... that is not managing a default device...
		ITargetDevicePtr TargetDevice = DeviceService->GetDevice();

		if (TargetDevice.IsValid() && TargetDevice->IsDefault())
		{
			continue;
		}

		// ... that should be automatically restarted next time...
		const bool* Shared = StartupServices.Find(DeviceName);

		if ((Shared != nullptr) || DeviceService->IsRunning())
		{
			// ... generate an entry in the INI file
			FString ServiceString = FString::Printf(TEXT("DeviceName=\"%s\",Shared=%s"),
				*DeviceName,
				((Shared != nullptr) && *Shared) ? TEXT("true") : TEXT("false")
			);

			ServiceStrings.Add(ServiceString);
		}
	}

	// save configuration
	GConfig->SetArray(TEXT("TargetDeviceServices"), TEXT("StartupServices"), ServiceStrings, GEngineIni);
	GConfig->Flush(false, GEngineIni);
}


void FTargetDeviceServiceManager::ShutdownTargetPlatforms()
{
	ITargetPlatformControls::OnDeviceDiscovered().RemoveAll(this);
	ITargetPlatformControls::OnDeviceLost().RemoveAll(this);
}


/* FTargetDeviceServiceManager callbacks
 *****************************************************************************/

void FTargetDeviceServiceManager::HandleMessageBusShutdown()
{
	MessageBusPtr.Reset();
}


void FTargetDeviceServiceManager::HandleTargetPlatformDeviceDiscovered(ITargetDeviceRef DiscoveredDevice)
{
	AddTargetDevice(DiscoveredDevice);	
}


void FTargetDeviceServiceManager::HandleTargetPlatformDeviceLost(ITargetDeviceRef LostDevice)
{
	RemoveTargetDevice(LostDevice);	
}
