// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"

#include "ITargetDeviceServiceManager.h"

class IMessageBus;
class ITargetDeviceService;


/**
 * Implements a target device service manager.
 *
 * Services that were running at the 
 */
class FTargetDeviceServiceManager
	: public ITargetDeviceServiceManager
{
public:

	/** Default constructor. */
	FTargetDeviceServiceManager();

	/** Destructor. */
	~FTargetDeviceServiceManager();

public:

	//~ ITargetDeviceServiceManager interface

	virtual bool AddStartupService(const FString& DeviceName) override;
	virtual int32 GetServices(TArray<TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe>>& OutServices) override;
	virtual void RemoveStartupService(const FString& DeviceName) override;

	DECLARE_DERIVED_EVENT(FTargetDeviceServiceManager, ITargetDeviceServiceManager::FOnTargetDeviceServiceAdded, FOnTargetDeviceServiceAdded);
	virtual FOnTargetDeviceServiceAdded& OnServiceAdded() override
	{
		return ServiceAddedDelegate;
	}

	DECLARE_DERIVED_EVENT(FTargetDeviceServiceManager, ITargetDeviceServiceManager::FOnTargetDeviceServiceRemoved, FOnTargetDeviceServiceRemoved);
	virtual FOnTargetDeviceServiceRemoved& OnServiceRemoved() override
	{
		return ServiceRemovedDelegate;
	}

protected:

	/**
	 * Adds a device service for the given device.
	 *
	 * @param DeviceName The name to of the device to add.
	 * @return the created or preexisting service, nullptr if creation failed.
	 */
	TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe> AddService(const FString& DeviceName);

	/**
	 * Adds a target device, a device service will be created if needed.
	 *
	 * @param InDevice The target device to add
	 * @return true if the device was added to the device service, false if the service failed to create.
	 */
	bool AddTargetDevice(ITargetDevicePtr InDevice);

	/**
	 * Initializes target platforms callbacks.
	 *
	 * @see ShutdownTargetPlatforms
	 */
	void InitializeTargetPlatforms();

	/**
	 * Loads all settings from disk.
	 *
	 * @see SaveSettings
	 */
	void LoadSettings();

	/**
	 * Removes the device service for the given target device.
	 *
	 * @param DeviceName The name of the device to remove the service for.
	 */
	void RemoveService(const FString& DeviceName);

	/**
	 * Removes the given target device, if that is the last target device in the device service the entire service is removed.
	 *
	 * @param InDevice The target device to remove.
	 */
	void RemoveTargetDevice(ITargetDevicePtr InDevice);

	/**
	 * Saves all settings to disk.
	 *
	 * @see LoadSettings
	 */
	void SaveSettings();

	/**
	 * Shuts down target platforms callbacks.
	 *
	 * @see InitializeTargetPlatforms
	 */
	void ShutdownTargetPlatforms();

private:

	/** Callback for handling message bus shutdowns. */
	void HandleMessageBusShutdown();

	/** Callback for discovered target devices. */
	void HandleTargetPlatformDeviceDiscovered(ITargetDeviceRef DiscoveredDevice);

	/** Callback for lost target devices. */
	void HandleTargetPlatformDeviceLost(ITargetDeviceRef LostDevice);

private:

	/** Holds a critical section object. */
	FCriticalSection CriticalSection;

	/** Holds the list of locally managed device services. */
	TMap<FString, TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe>> DeviceServices;

	/** Holds a weak pointer to the message bus. */
	TWeakPtr<IMessageBus, ESPMode::ThreadSafe> MessageBusPtr;

	/** Holds the collection of identifiers for devices that start automatically (shared/unshared). */
	TMap<FString, bool> StartupServices;

private:

	/** Holds a delegate that is invoked when a target device service was added. */
	FOnTargetDeviceServiceAdded ServiceAddedDelegate;

	/** Holds a delegate that is invoked when a target device service was removed. */
	FOnTargetDeviceServiceRemoved ServiceRemovedDelegate;
};
