// Copyright Epic Games, Inc. All Rights Reserved.

#include "TargetDeviceServicesPrivate.h"

#include "CoreTypes.h"
#include "Modules/ModuleManager.h"

#include "ITargetDeviceServicesModule.h"
#include "Proxies/TargetDeviceProxyManager.h"
#include "Services/TargetDeviceServiceManager.h"


DEFINE_LOG_CATEGORY(TargetDeviceServicesLog);


/**
 * Implements the TargetDeviceServices module.
 */
class FTargetDeviceServicesModule
	: public ITargetDeviceServicesModule
{
public:

	//~ ITargetDeviceServicesModule interface

	virtual TSharedRef<ITargetDeviceProxyManager> GetDeviceProxyManager() override
	{
		if (!DeviceProxyManagerSingleton.IsValid())
		{
			DeviceProxyManagerSingleton = MakeShareable(new FTargetDeviceProxyManager());
		}

		return DeviceProxyManagerSingleton.ToSharedRef();
	}

	virtual TSharedRef<ITargetDeviceServiceManager> GetDeviceServiceManager() override
	{
		if (!DeviceServiceManagerSingleton.IsValid())
		{
			DeviceServiceManagerSingleton = MakeShareable(new FTargetDeviceServiceManager());
		}

		return DeviceServiceManagerSingleton.ToSharedRef();
	}

private:

	/** Holds the device proxy manager singleton. */
	TSharedPtr<FTargetDeviceProxyManager> DeviceProxyManagerSingleton;

	/** Holds the device service manager singleton. */
	TSharedPtr<FTargetDeviceServiceManager> DeviceServiceManagerSingleton;
};


IMPLEMENT_MODULE(FTargetDeviceServicesModule, TargetDeviceServices);
