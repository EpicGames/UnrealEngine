// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"


/** Declares the module's log category. */
DECLARE_LOG_CATEGORY_EXTERN(TargetDeviceServicesLog, Log, All);
