// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class ITargetDeviceProxy;

struct FGuid;

/**
 * Interface for target device proxies.
 */
class ITargetDeviceProxy
{
public:

	/**
	 * Checks whether the device can launch multiple games or applications simultaneously.
	 *
	 * @return true if launching multiple instances is supported, false otherwise.
	 */
	virtual const bool CanMultiLaunch() const = 0;

	/**
	 * Checks whether this device can be powered off remotely.
	 *
	 * @return true if the device can be powered off, false otherwise.
	 */
	virtual bool CanPowerOff() const = 0;

	/**
	 * Checks whether this device can be powered on remotely.
	 *
	 * @return true if the device can be powered on, false otherwise.
	 */
	virtual bool CanPowerOn() const = 0;

	/**
	 * Checks whether this device can be rebooted remotely.
	 *
	 * @return true if the device can be rebooted, false otherwise.
	 */
	virtual bool CanReboot() const = 0;

	/**
	 * Checks whether this device's target platform supports variants.
	 *
	 * @return true if the device's target platform supports variants, false otherwise.
	 */
	virtual bool CanSupportVariants() const = 0;

	/**
	 * Gets the number of variants this device supports
	 *
	 * @return The number of variants supported.
	 */
	virtual int32 GetNumVariants() const = 0;

	/**
	 * Gets the list of variants this device supports
	 *
	 * @param OutVariants This list will be populated with the unique variant names. 
	 * @return The number of variants in the list.
	 */
	virtual int32 GetVariants(TArray<FName>& OutVariants) const = 0;

	/**
	 * Checks whether this device proxy contains a variant
	 *
	 * @param InVariant Variant to check for.
	 * @return true is this variant is found.
	 */
	virtual bool HasVariant(FName InVariant) const = 0;

	/**
	 * Gets the variant name of the target device.
	 *
	 * @param InDeviceId target device id to get the variant for.
	 * @return The target device variant name.
	 */
	virtual FName GetTargetDeviceVariant(const FString& InDeviceId) const = 0;

	/**
	 * Checks whether this device proxy contains a variant for the provided target device.
	 *
	 * @param InDeviceId DeviceId to check for a variant.
	 * @return true if this device has a variant of the provided DeviceId.
	 */
	virtual bool HasDeviceId(const FString& InDeviceId) const = 0;

	/**
	 * Gets the identifier of the device.
	 *
	 * @param InVariant name of the variant to return the device identifier for, If InVariant == NAME_None you'll get back the default target device identifier.
	 * @return The target device identifier.
	 */
	virtual const FString GetTargetDeviceId(FName InVariant) const = 0;

	/**
	* Gets a list of device identifiers for an aggregate (All_<platform>_devices_on_<host>) proxy
	*
	* @param InVariant name of the variant to return the device identifier for, If InVariant == NAME_None you'll get back the default target device identifiers.
	* @param OutDeviceIds The list of device identifiers.
	*/
	virtual const TSet<FString>& GetTargetDeviceIds(FName InVariant) const = 0;

	/**
	* Checks whether this device proxy contains a variant for the provided platform.
	*
	* @param InTargetPlatformName Target Platform Name for the target device to check for as a variant of this device.
	* @return true if this device has a variant of the provided Target Platform .
	*/
	virtual bool HasTargetPlatform(FName InTargetPlatformId) const = 0;

	/**
	 * Gets the target platform of the device variant.
	 *
	 * @param InVariant Variant to get the platform name for.
	 * @return Target Platform Name string.
	 */
	virtual FString GetTargetPlatformName(FName InVariant) const = 0;

	/**
	 * Gets the target platform of the device variant.
	 *
	 * @param InVariant Variant to get the platform id for.
	 * @return Target Platform Id.
	 */
	virtual FName GetTargetPlatformId(FName InVariant) const = 0;

	/**
	 * Gets the Vanilla platform of the device variant.
	 *
	 * @param InVariant Variant to get the platform id for.
	 * @return Vanilla Platform Id.
	 */
	virtual FName GetVanillaPlatformId(FName InVariant) const = 0;

	/**
	 * Gets the Vanilla platform of the device variant.
	 *
	 * @param InVariant Variant to get the vanilla platform name for.
	 * @return Platform Display Name string.
	 */
	virtual FText GetPlatformDisplayName(FName InVariant) const = 0;

	/**
	 * Gets the name of the host machine that claimed the device.
	 *
	 * @return Host name string.
	 */
	virtual const FString& GetHostName() const = 0;
	 
	/**
	 * Gets the name of the user that claimed the device.
	 *
	 * @return User name.
	 */
	virtual const FString& GetHostUser() const = 0;

	/**
	 * Gets the name of the (device's) user that is logged in on a device.
	 *
	 * @return Device user name.
	 */
	virtual const FString& GetDeviceUser() const = 0;

	/**
	 * Gets the password of the (device's) user that is logged in on a device.
	 *
	 * @return Device user password.
	 */
	virtual const FString& GetDeviceUserPassword() const = 0;

	/**
	 * Gets the device make (i.e. Apple or Sony).
	 *
	 * @return Device make.
	 */
	virtual const FString& GetMake() const = 0;

	/**
	 * Gets the device model (i.e. PS3 or XBox).
	 *
	 * @return Device model.
	 */
	virtual const FString& GetModel() const = 0;
	 
	/**
	 * Gets the name of the device (i.e. network name or IP address).
	 *
	 * @return Device name string.
	 */
	virtual const FString& GetName() const = 0;

	/**
	 * Gets the device type (i.e. Console, PC or Mobile).
	 *
	 * @return Device type.
	 */
	virtual const FString& GetType() const = 0;

	/**
	 * Gets the OS Version of the device
	 *
	 * @return Device OS Version.
	 */
	virtual const FString& GetOSVersion() const = 0;

	/**
	 * Gets the architecture of the device
	 *
	 * @return Device Architecture.
	 */
	virtual FStringView GetArchitecture() const
	{
		return {};
	}

	/**
	 * Gets the type of connection for this device.
	 *
	 * @return Device type.
	 */
	virtual const FString& GetConnectionType() const = 0;

	/**
	 * Checks whether the device is currently connected.
	 *
	 * @return true if the device is connected, false otherwise.
	 */
	virtual bool IsConnected() const = 0;

	/**
	 * Checks whether the device is authorized.
	 *
	 * @return true if the device is authorized, false otherwise.
	 */
	virtual bool IsAuthorized() const = 0;

	/**
	 * Checks whether this device is being shared with other users.
	 *
	 * @return true if the device is being shared, false otherwise.
	 */
	virtual bool IsShared() const = 0;

	/**
	 * Checks whether this device is being simulated on a Simulator
	 *
	 * @return true if the device is being simulated, false otherwise.
	 */
	virtual bool IsSimulated() const = 0;
	
	/**
	* Cancel the application running on the device
	 * @param InVariant Variant of the device
	 * @param ProcessIdentifier The bundle id
   	 * @return true if the launch has been successfully terminated, false otherwise.
	*/
	virtual bool TerminateLaunchedProcess(FName InVariant, const FString& ProcessIdentifier) = 0;

	/**
	* Checks if this is an aggregate (All_<platform>_devices_on_<host>) proxy.
	*
	* @return true if the device is aggregated, false otherwise.
	*/
	virtual bool IsAggregated() const { return false;  }

public:


	/**
	 * Powers off the device.
	 *
	 * @param Force Whether to force powering off.
	 * @see PowerOn, Reboot
	 */
	virtual void PowerOff(bool Force) = 0;

	/**
	 * Powers on the device.
	 *
	 * @see PowerOff, Reboot
	 */
	virtual void PowerOn() = 0;

	/**
	 * Reboots the device.
	 *
	 * @see PowerOff, PowerOn
	 */
	virtual void Reboot() = 0;

public:

	/** Virtual destructor. */
	virtual ~ITargetDeviceProxy() = default;
};
