// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FString;
class ITargetDeviceProxy;
class ITargetDeviceProxyManager;
class ITargetDeviceServiceManager;

/**
 * Interface for device proxy managers.
 */
class ITargetDeviceProxyManager
{
public:
	
	/**
	 * Finds or adds a device proxy for the specified device name.
	 *
	 * @param Name The name of the device to create the proxy for.
	 * @return The device proxy.
	 * @see FindProxy, FindProxyDeviceForTargetDevice, GetProxies
	 */
	virtual TSharedRef<ITargetDeviceProxy> FindOrAddProxy(const FString& Name) = 0;

	/**
	 * Finds the device proxy for the specified device name.
	 *
	 * @param Name The name of the device to create the proxy for.
	 * @return The device proxy, or nullptr if it couldn't be found.
	 * @see FindOrAddProxy, FindProxyDeviceForTargetDevice, GetProxies
	 */
	virtual TSharedPtr<ITargetDeviceProxy> FindProxy(const FString& Name) = 0;

	/**
	 * Finds the device proxy for the specified target device id.
	 *
	 * @param DeviceId The identifier of the target device to create the proxy for.
	 * @return The device proxy, or nullptr if it couldn't be found.
	 * @see FindOrAddProxy, FindProxy, GetProxies
	 */
	virtual TSharedPtr<ITargetDeviceProxy> FindProxyDeviceForTargetDevice(const FString& DeviceId) = 0;

	/**
	 * Gets a list of devices found by the device discovery.
	 *
	 * @param PlatformName The the name of the target platform to get proxies for (or empty string for all proxies).
	 * @param IncludeUnshared Whether to include devices that are not being shared with the local user.
	 * @param OutProxies Will hold the list of devices found by the locator.
	 * @see FindOrAddProxy, FindProxy, FindProxyDeviceForTargetDevice
	 */
	virtual void GetProxies(FName TargetPlatformName, bool IncludeUnshared, TArray<TSharedPtr<ITargetDeviceProxy>>& OutProxies) = 0;

	/**
	* Gets a list of proxies created by the device discovery, including the "All device" proxies
	*
	* @param PlatformName The the name of the target platform to get proxies for (or empty string for all proxies).
	* @param OutProxies Will hold the list of devices found by the locator.
	* @see FindOrAddProxy, FindProxy, FindProxyDeviceForTargetDevice
	*/
	virtual void GetAllProxies(FName TargetPlatformName, TArray<TSharedPtr<ITargetDeviceProxy>>& OutProxies) = 0;

public:

	/**
	 * Gets an event delegate that is executed when a target device proxy was added.
	 *
	 * @return The event delegate.
	 */
	DECLARE_EVENT_OneParam(ITargetDeviceServiceManager, FOnTargetDeviceProxyAdded, const TSharedRef<ITargetDeviceProxy>& /*AddedProxy*/);
	virtual FOnTargetDeviceProxyAdded& OnProxyAdded() = 0;

	/**
	 * Gets an event delegate that is executed when a target device proxy was removed.
	 *
	 * @return The event delegate.
	 */
	DECLARE_EVENT_OneParam(ITargetDeviceServiceManager, FOnTargetDeviceProxyRemoved, const TSharedRef<ITargetDeviceProxy>& /*RemovedProxy*/);
	virtual FOnTargetDeviceProxyRemoved& OnProxyRemoved() = 0;

public:

	/** Virtual destructor. */
	virtual ~ITargetDeviceProxyManager() = default;
};
