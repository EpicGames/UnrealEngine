// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"
#include "Interfaces/ITargetDevice.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class ITargetDevice;
class ITargetDeviceService;


/** Type definition for shared pointers to instances of ITargetDeviceService. */
typedef TSharedPtr<class ITargetDeviceService, ESPMode::ThreadSafe> ITargetDeviceServicePtr;

/** Type definition for shared references to instances of ITargetDeviceService. */
typedef TSharedRef<class ITargetDeviceService, ESPMode::ThreadSafe> ITargetDeviceServiceRef;


/**
 * Interface for target device services.
 *
 * Device services make locally connected or discovered network target devices
 * available over the network. Remote clients communicate with device services
 * through a target device proxy.
 *
 * @see ITargetDeviceProxy
 */
class ITargetDeviceService
{
public:

	/**
	 * Adds a flavor to this device.
	 *
	 * @return true if the service can start, false otherwise.
	 */
	virtual void AddTargetDevice(TSharedPtr<ITargetDevice, ESPMode::ThreadSafe> InDevice) = 0;

	/**
	 * Checks whether this service can start.
	 *
	 * A service can be started if it has a valid device and the device is connected.
	 * This method does not take into account whether the service is already started.
	 *
	 * @return true if the service can start, false otherwise.
	 */
	virtual bool CanStart(FName InFlavor = NAME_None) const = 0;

	/**
	 * Gets the name of the host that has a claim on the device.
	 *
	 * @return Host name string.
	 */
	virtual const FString& GetClaimHost() = 0;

	/**
	 * Gets the name of the user that has a claim on the device.
	 *
	 * @return User name string.
	 */
	virtual const FString& GetClaimUser() = 0;

	/**
	 * Gets the target device for the supplied flavor, if no flavor is specified then the default device flavor is returned.
	 *
	 * @return A pointer to the target device.
	 */
	virtual TSharedPtr<ITargetDevice, ESPMode::ThreadSafe> GetDevice(FName InFlavor = NAME_None) const = 0;

	/**
	 * Gets the name of the device that this service exposes.
	 *
	 * @return The human readable device name.
	 */
	virtual FString GetDeviceName() const = 0;

	/**
	 * Gets the name of the platform of device that this service exposes.
	 *
	 * @return The platform name.
	 */
	virtual FName GetDevicePlatformName() const = 0;

	/**
	 * Gets the display name of the platform of device that this service exposes.
	 *
	 * @return The human readable platform name.
	 */
	virtual FString GetDevicePlatformDisplayName() const = 0;

	/**
	 * Checks whether the service is currently running.
	 *
	 * @return true if the service is running, false otherwise.
	 */
	virtual bool IsRunning() const = 0;

	/**
	 * Checks whether the device is being shared with other users.
	 *
	 * If a device is shared, the device proxies of all users on the network
	 * can discover and use the device.
	 *
	 * @return true if the device is shared, false otherwise.
	 * @see SetShared
	 */
	virtual bool IsShared() const = 0;

	/**
	 * Gets number of target devices.
	 *
	 * @return number of target devices in this device service.
	 */
	virtual int32 NumTargetDevices() = 0;

	/**
	 * Removes a flavor from this device.
	 *
	 * @return true if the service can start, false otherwise.
	 */
	virtual void RemoveTargetDevice(TSharedPtr<ITargetDevice, ESPMode::ThreadSafe> InDevice) = 0;

	/**
	 * Sets whether the device should be shared with other users.
	 *
	 * If a device is shared, the device proxies of all users on the network
	 * can discover and use the device.
	 *
	 * @param InShared Indicates whether device sharing is enabled.
	 * @see IsShared
	 */
	virtual void SetShared(bool InShared) = 0;

	/**
	 * Starts the service.
	 *
	 * @return true if the service has been started, false otherwise.
	 * @see IsRunning, Stop
	 */
	virtual bool Start() = 0;

	/**
	 * Stops the service.
	 *
	 * @see IsRunning, Stop
	 */
	virtual void Stop() = 0;

public:

	/** Virtual destructor. */
	virtual ~ITargetDeviceService() = default;
};
