// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "ITargetDeviceService.h"
#include "Templates/SharedPointer.h"

class FString;
class ITargetDeviceService;

/**
 * Interface for target device service managers.
 */
class ITargetDeviceServiceManager
{
public:

	/**
	 * Adds a service to the list of services that are started automatically.
	 *
	 * A preliminary device name may be assigned to services that expose devices which
	 * could not be discovered automatically or are currently unavailable. This name will
	 * be replaced with the actual device name once the physical device becomes available.
	 *
	 * @param DeviceName The device name for the service to add to the startup list
	 * @return true if the service added, false otherwise.
	 * @see GetServices, RemoveStartupService
	 */
	virtual bool AddStartupService(const FString& DeviceName) = 0;

	/**
	 * Gets the collection of target device services managed by this instance.
	 *
	 * @param OutServices Will contain the collection of services.
	 * @return The number of services returned.
	 */
	virtual int32 GetServices(TArray<TSharedPtr<ITargetDeviceService, ESPMode::ThreadSafe>>& OutServices) = 0;

	/**
	 * Removes a service from the list of services that are started automatically.
	 *
	 * @param DeviceName The device name for the service to remove from the startup list.
	 * @see AddStartupService, GetServices
	 */
	virtual void RemoveStartupService(const FString& DeviceName) = 0;

public:

	/**
	 * Gets an event delegate that is executed when a target device service was added.
	 *
	 * @return The event delegate.
	 */
	DECLARE_EVENT_OneParam(ITargetDeviceServiceManager, FOnTargetDeviceServiceAdded, const ITargetDeviceServiceRef& /*AddedService*/);
	virtual FOnTargetDeviceServiceAdded& OnServiceAdded() = 0;

	/**
	 * Gets an event delegate that is executed when a target device service was removed.
	 *
	 * @return The event delegate.
	 */
	DECLARE_EVENT_OneParam(ITargetDeviceServiceManager, FOnTargetDeviceServiceRemoved, const ITargetDeviceServiceRef& /*RemovedService*/);
	virtual FOnTargetDeviceServiceRemoved& OnServiceRemoved() = 0;

public:

	/** Virtual destructor. */
	virtual ~ITargetDeviceServiceManager() = default;
};
