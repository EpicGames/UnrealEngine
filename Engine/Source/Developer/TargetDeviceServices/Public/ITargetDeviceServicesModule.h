// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ITargetDeviceProxyManager.h"
#include "ITargetDeviceServiceManager.h"
#include "Modules/ModuleInterface.h"


/**
 * Interface for target device services modules.
 */
class ITargetDeviceServicesModule
	: public IModuleInterface
{
public:

	/**
	 * Gets the target device proxy manager.
	 *
	 * @return The device proxy manager.
	 * @see GetDeviceServiceManager
	 */
	virtual TSharedRef<ITargetDeviceProxyManager> GetDeviceProxyManager() = 0;

	/**
	 * Gets the target device service manager.
	 *
	 * @return The device service manager.
	 * @see GetDeviceProxyManager
	 */
	virtual TSharedRef<ITargetDeviceServiceManager> GetDeviceServiceManager() = 0;

public:

	/** Virtual destructor. */
	virtual ~ITargetDeviceServicesModule() = default;
};
