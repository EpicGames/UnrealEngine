// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"

#include "TargetDeviceServiceMessages.generated.h"



/* Application terminate process messages
*****************************************************************************/

/**
* Implements a message for terminating an application running on the device.
*
* @see FTargetDeviceServiceTerminateLaunchedProcess
*/
USTRUCT()
struct FTargetDeviceServiceTerminateLaunchedProcess
{
	GENERATED_USTRUCT_BODY()

	/** Holds the variant identifier of the target device to use. */
	UPROPERTY(EditAnywhere, Category = "Message")
	FName Variant;

	/**
	* Holds the identifier of the application to launch.
	*
	* The semantics of this identifier are target platform specific. In some cases it may be
	* a GUID, in other cases it may be the path to the application or some other means of
	* identifying the application. Application identifiers are returned from target device
	* services as result of successful deployment transactions.
	*/
	UPROPERTY(EditAnywhere, Category = "Message")
	FString AppID;

	/** Default constructor. */
	FTargetDeviceServiceTerminateLaunchedProcess() { }

	/** Creates and initializes a new instance. */
	FTargetDeviceServiceTerminateLaunchedProcess(FName InVariant, const FString& InAppId)
		: Variant(InVariant)
		, AppID(InAppId)
	{ }
};


/* Device claiming messages
 *****************************************************************************/

/**
 * Implements a message that is sent when a device is already claimed by someone else.
 *
 * @see FTargetDeviceClaimDropped
 * @see FTargetDeviceClaimRequest
 */
USTRUCT()
struct FTargetDeviceClaimDenied
{
	GENERATED_USTRUCT_BODY()

	/** Holds the identifier of the device that is already claimed. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString DeviceName;

	/** Holds the name of the host computer that claimed the device. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString HostName;

	/** Holds the name of the user that claimed the device. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString HostUser;


	/** Default constructor. */
	FTargetDeviceClaimDenied() { }

	/** Creates and initializes a new instance. */
	FTargetDeviceClaimDenied(const FString& InDeviceName, const FString& InHostName, const FString& InHostUser)
		: DeviceName(InDeviceName)
		, HostName(InHostName)
		, HostUser(InHostUser)
	{ }
};


/**
 * Implements a message that is sent when a service claimed a device.
 *
 * @see FTargetDeviceClaimDenied
 * @see FTargetDeviceClaimDropped
 */
USTRUCT()
struct FTargetDeviceClaimed
{
	GENERATED_USTRUCT_BODY()

	/** Holds the identifier of the device that is being claimed. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString DeviceName;

	/** Holds the name of the host computer that is claiming the device. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString HostName;

	/** Holds the name of the user that is claiming the device. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString HostUser;


	/** Default constructor. */
	FTargetDeviceClaimed() { }

	/** Creates and initializes a new instance. */
	FTargetDeviceClaimed(const FString& InDeviceName, const FString& InHostName, const FString& InHostUser)
		: DeviceName(InDeviceName)
		, HostName(InHostName)
		, HostUser(InHostUser)
	{ }
};


/**
 * Implements a message that is sent when a device is no longer claimed.
 *
 * @see FTargetDeviceClaimDenied, FTargetDeviceClaimRequest
 */
USTRUCT()
struct FTargetDeviceUnclaimed
{
	GENERATED_USTRUCT_BODY()

	/** Holds the identifier of the device that is no longer claimed. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString DeviceName;

	/** Holds the name of the host computer that had claimed the device. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString HostName;

	/** Holds the name of the user that had claimed the device. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString HostUser;


	/** Default constructor. */
	FTargetDeviceUnclaimed() { }

	/** Creates and initializes a new instance. */
	FTargetDeviceUnclaimed(const FString& InDeviceName, const FString& InHostName, const FString& InHostUser)
		: DeviceName(InDeviceName)
		, HostName(InHostName)
		, HostUser(InHostUser)
	{ }
};


/* Device discovery messages
 *****************************************************************************/

/**
 * Implements a message for discovering target device services on the network.
 */
USTRUCT()
struct FTargetDeviceServicePing
{
	GENERATED_USTRUCT_BODY()

	/** Holds the name of the user who generated the ping. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString HostUser;


	/** Default constructor. */
	FTargetDeviceServicePing() { }

	/** Creates and initializes a new instance. */
	FTargetDeviceServicePing(const FString& InHostUser)
		: HostUser(InHostUser)
	{ }
};


/**
* Struct for a flavor's information
*/
USTRUCT()
struct FTargetDeviceVariant
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category="Variant")
	FString DeviceID;

	UPROPERTY(EditAnywhere, Category="Variant")
	FName VariantName;

	UPROPERTY(EditAnywhere, Category="Variant")
	FString TargetPlatformName;

	UPROPERTY(EditAnywhere, Category="Variant")
	FName TargetPlatformId;

	UPROPERTY(EditAnywhere, Category="Variant")
	FName VanillaPlatformId;

	UPROPERTY(EditAnywhere, Category="Variant")
	FString PlatformDisplayName;
};


/**
 * Implements a message that is sent in response to target device service discovery messages.
 */
USTRUCT()
struct FTargetDeviceServicePong
{
	GENERATED_USTRUCT_BODY()

	/** Holds a flag indicating whether the device is currently connected. */
	UPROPERTY(EditAnywhere, Category="Message")
	bool Connected = false;

	/** Holds a flag indicating whether the device is authorized. */
	UPROPERTY(EditAnywhere, Category="Message")
	bool Authorized = false;

	/** Holds the name of the host computer that the device is attached to. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString HostName;

	/** Holds the name of the user under which the host computer is running. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString HostUser;

	/** Holds the make of the device, i.e. Microsoft or Sony. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Make;

	/** Holds the model of the device. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Model;

	/** Holds the human readable name of the device, i.e "Bob's XBox'. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Name;

	/** Holds the name of the user that we log in to remote device as, i.e "root". */
	UPROPERTY(EditAnywhere, Category="Message")
	FString DeviceUser;

	/** Holds the password of the user that we log in to remote device as, i.e "12345". */
	UPROPERTY(EditAnywhere, Category="Message")
	FString DeviceUserPassword;

	/** Holds a flag indicating whether this device is shared with other users on the network. */
	UPROPERTY(EditAnywhere, Category="Message")
	bool Shared = false;

	/** Holds a flag indicating whether the device supports running multiple application instances in parallel. */
	UPROPERTY(EditAnywhere, Category="Message")
	bool SupportsMultiLaunch = false;

	/** Holds a flag indicating whether the device can be powered off. */
	UPROPERTY(EditAnywhere, Category="Message")
	bool SupportsPowerOff = false;

	/** Holds a flag indicating whether the device can be powered on. */
	UPROPERTY(EditAnywhere, Category="Message")
	bool SupportsPowerOn = false;

	/** Holds a flag indicating whether the device can be rebooted. */
	UPROPERTY(EditAnywhere, Category="Message")
	bool SupportsReboot = false;

	/** Holds a flag indicating whether the device's target platform supports variants. */
	UPROPERTY(EditAnywhere, Category="Message")
	bool SupportsVariants = false;

	/** Holds the device type. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Type;

	/** Holds the device OS Version. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString OSVersion;

	/** Holds the device architecture. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Architecture;

	/** Holds the connection type. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString ConnectionType;

	/** Holds the variant name of the default device. */
	UPROPERTY(EditAnywhere, Category="Message")
	FName DefaultVariant;

	/** List of the Flavors this service supports */
	UPROPERTY(EditAnywhere, Category="Message")
	TArray<FTargetDeviceVariant> Variants;

	/** Flag for the "All devices" proxy. */
	UPROPERTY(EditAnywhere, Category = "Message")
	bool Aggregated = true;

	/** Holds the name of "All devices" proxy. */
	UPROPERTY(EditAnywhere, Category = "Message")
	FString AllDevicesName;

	/** Holds the default variant name of "All devices" proxy. */
	UPROPERTY(EditAnywhere, Category = "Message")
	FName AllDevicesDefaultVariant;
};


/* Miscellaneous messages
 *****************************************************************************/

/**
 * Implements a message for powering on a target device.
 */
USTRUCT()
struct FTargetDeviceServicePowerOff
{
	GENERATED_USTRUCT_BODY()

	/**
	 * Holds a flag indicating whether the power-off should be enforced.
	 *
	 * If powering off is not enforced, if may fail if some running application prevents it.
	 */
	UPROPERTY(EditAnywhere, Category="Message")
	bool Force;

	/** Holds the name of the user that wishes to power off the device. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Operator;


	/** Default constructor. */
	FTargetDeviceServicePowerOff() : Force(false) { }

	/** Creates and initializes a new instance. */
	FTargetDeviceServicePowerOff(const FString& InOperator, bool InForce)
		: Force(InForce)
		, Operator(InOperator)
	{ }
};


/**
 * Implements a message for powering on a target device.
 */
USTRUCT()
struct FTargetDeviceServicePowerOn
{
	GENERATED_USTRUCT_BODY()

	/** Holds the name of the user that wishes to power on the device. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Operator;


	/** Default constructor. */
	FTargetDeviceServicePowerOn() { }

	/** Creates and initializes a new instance. */
	FTargetDeviceServicePowerOn(const FString& InOperator)
		: Operator(InOperator)
	{ }
};


/**
 * Implements a message for rebooting a target device.
 */
USTRUCT()
struct FTargetDeviceServiceReboot
{
	GENERATED_USTRUCT_BODY()

	/** Holds the name of the user that wishes to reboot the device. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Operator;


	/** Default constructor. */
	FTargetDeviceServiceReboot() { }

	/** Creates and initializes a new instance. */
	FTargetDeviceServiceReboot(const FString& InOperator)
		: Operator(InOperator)
	{ }
};
