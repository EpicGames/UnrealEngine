// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class TargetDeviceServices : ModuleRules
	{
		public TargetDeviceServices(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"TargetPlatform",
					"DesktopPlatform",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"CoreUObject",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Messaging",
					"MessagingCommon",
				});
		}
	}
}
